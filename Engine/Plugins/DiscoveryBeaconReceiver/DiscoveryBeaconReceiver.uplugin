{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Discovery Beacon Receiver",
	"Description": "Listens for messages on a multicast socket and replies with information about the engine.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DiscoveryBeaconReceiver",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ]
		}
	]
}