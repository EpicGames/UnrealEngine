// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiscoveryBeaconReceiver.h"

#include "Common/UdpSocketBuilder.h"
#include "DiscoveryBeaconReceiverModule.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"

FDiscoveryBeaconReceiver::FDiscoveryBeaconReceiver(const FString& InDescription, const TArray<uint8>& InProtocolIdentifier, uint8 InProtocolVersion)
	: Description(InDescription), ProtocolIdentifier(InProtocolIdentifier), ProtocolVersion(InProtocolVersion), Guid(FGuid::NewGuid())
{
}

void FDiscoveryBeaconReceiver::Startup()
{
	if (bIsRunning)
	{
		return;
	}

	FIPv4Address DiscoveryAddress;
	if (!GetDiscoveryAddress(DiscoveryAddress))
	{
		UE_LOGF(LogDiscoveryBeaconReceiver, Warning, "No discovery address provided for %ls", *Description);
		return;
	}

	const int32 DiscoveryPort = GetDiscoveryPort();
	if (DiscoveryPort < 0)
	{
		UE_LOGF(LogDiscoveryBeaconReceiver, Warning, "No valid discovery port provided for %ls (got %d)", *Description, DiscoveryPort);
		return;
	}

	Socket = FUdpSocketBuilder(Description)
		.AsNonBlocking()
		.AsReusable()
		.BoundToPort(DiscoveryPort)
		.WithMulticastLoopback()
		.WithReceiveBufferSize(256);

	if (!Socket)
	{
		UE_LOGF(LogDiscoveryBeaconReceiver, Warning, "%ls failed to create multicast socket", *Description);
		return;
	}

	bIsRunning = true;

	// Listen for multicast packets on all interfaces.
	// Note: If a new interface appears later on, the socket will need to join it in order to receive multicast packets from it.
	{
		TSharedRef<FInternetAddr> DiscoveryInternetAddr = FIPv4Endpoint(DiscoveryAddress, 0).ToInternetAddr(); // Note: Port will be ignored by JoinMulticastGroup

		TArray<TSharedPtr<FInternetAddr>> LocapIps;
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(LocapIps);

		for (TSharedPtr<FInternetAddr>& LocalIp : LocapIps)
		{
			if (!LocalIp.IsValid())
			{
				continue;
			}

			const bool bJoinedGroup = Socket->JoinMulticastGroup(*DiscoveryInternetAddr, *LocalIp);

			if (bJoinedGroup)
			{
				UE_LOGF(LogDiscoveryBeaconReceiver, Log,
					"%ls joined multicast group '%ls' on detected local interface '%ls'",
					*Description, *DiscoveryAddress.ToString(), *LocalIp->ToString(false));
			}
			else
			{
				UE_LOGF(LogDiscoveryBeaconReceiver, Warning,
					"%ls failed to join multicast group '%ls' on detected local interface '%ls'",
					*Description, *DiscoveryAddress.ToString(), *LocalIp->ToString(false));
			}
		}
	}

	Thread.Reset(FRunnableThread::Create(this, *FString::Printf(TEXT("%sResponderThread"), *Description), 0, TPri_Normal, FPlatformAffinity::GetPoolThreadMask()));
}

void FDiscoveryBeaconReceiver::Shutdown()
{
	if (Thread)
	{
		Thread->Kill();
	}

	if (Socket)
	{
		ISocketSubsystem::Get()->DestroySocket(Socket);
		Socket = nullptr;
	}
}

bool FDiscoveryBeaconReceiver::Init()
{
	return true;
}

uint32 FDiscoveryBeaconReceiver::Run()
{
	const FTimespan WaitTime = FTimespan::FromSeconds(1.0);

	while (bIsRunning)
	{
		if (!Socket)
		{
			return 0;
		}

		Socket->Wait(ESocketWaitConditions::WaitForRead, WaitTime);
		ReceiveBeaconMessages();
	}

	return 0;
}

void FDiscoveryBeaconReceiver::Stop()
{
	bIsRunning = false;
}

void FDiscoveryBeaconReceiver::ReceiveBeaconMessages()
{
	if (!Socket)
	{
		return;
	}

	uint32 PendingDataSize;
	if (Socket->HasPendingData(PendingDataSize) && PendingDataSize > 0)
	{
		FArrayReader MessageData = FArrayReader(true);
		MessageData.SetNumUninitialized(PendingDataSize);

		int32 NumRead;
		TSharedRef<FInternetAddr> Source = ISocketSubsystem::Get()->CreateInternetAddr();
		if (Socket->RecvFrom(MessageData.GetData(), MessageData.Num(), NumRead, *Source) && NumRead > 0)
		{
			if (NumRead < MessageData.Num())
			{
				// Update container size to reflect the actual message (which only matters if it's smaller, since the message
				// will fill the container otherwise).
				// Don't allow shrinking since this could incur a move and we're just going to release the array anyway.
				MessageData.SetNum(NumRead, EAllowShrinking::No);
			}
			HandleBeaconMessage(MessageData, Source);
		}
	}
}

void FDiscoveryBeaconReceiver::HandleBeaconMessage(FArrayReader& MessageData, TSharedRef<FInternetAddr> Source)
{
	// Check that the protocol identifier matches. If not, this message is probably unrelated.
	for (const uint8 ExpectedByte : ProtocolIdentifier)
	{
		uint8 AppByte;
		MessageData << AppByte;

		if (AppByte != ExpectedByte)
		{
			return;
		}
	}

	// We don't do anything with this yet, but could use it to ignore beacons from apps with incompatible protocols
	uint8 BeaconProtocolVersion;
	MessageData << BeaconProtocolVersion;

	FArrayWriter Writer(true);

#if !PLATFORM_LITTLE_ENDIAN
	// App expects little-endian byte order
	Writer.SetByteSwapping(true);
#endif

	{
		Writer << (uint8&)ProtocolVersion;
		Writer << Guid;
	}

	if (!MakeBeaconResponse(BeaconProtocolVersion, MessageData, Writer))
	{
		return;
	}

	// Message wasn't fully consumed.
	if (!MessageData.AtEnd())
	{
		return;
	}

	int32 Sent;
	Socket->SendTo(Writer.GetData(), Writer.Num(), Sent, *Source);
}