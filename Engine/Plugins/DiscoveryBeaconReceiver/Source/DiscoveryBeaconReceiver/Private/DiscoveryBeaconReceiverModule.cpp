// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiscoveryBeaconReceiverModule.h"

#define LOCTEXT_NAMESPACE "FDiscoveryBeaconReceiverModule"

DEFINE_LOG_CATEGORY(LogDiscoveryBeaconReceiver);

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FDiscoveryBeaconReceiverModule, DiscoveryBeaconReceiver)
