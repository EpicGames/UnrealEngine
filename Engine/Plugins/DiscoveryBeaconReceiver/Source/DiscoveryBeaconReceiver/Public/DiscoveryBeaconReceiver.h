// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"

class FArrayReader;
class FBufferArchive;
class FInternetAddr;
class FSocket;
struct FIPv4Address;
typedef FBufferArchive FArrayWriter;

/**
 * Receives beacon messages on a multicast endpoint and replies with information to connect to the engine.
 * This allows remote apps to detect compatible Unreal instances on the local network.
 */
class DISCOVERYBEACONRECEIVER_API FDiscoveryBeaconReceiver
	: public FRunnable
{
public:
	/**
	 * Creates and initializes a new beacon receiver.
	 * @param InDescription A short name for this beacon for debugging purposes.
	 * @param InProtocolIdentifier A series of bytes that uniquely identify messages intended for this beacon receiver.
	 * @param InProtocolVersion The protocol version number with which to reply to beacon messages.
	 */
	FDiscoveryBeaconReceiver(const FString& InDescription, const TArray<uint8>& InProtocolIdentifier, uint8 InProtocolVersion);

	/** Destroy the beacon receiver. */
	virtual ~FDiscoveryBeaconReceiver() = default;

	/**
	 * Open a socket and start a thread listening for beacon messages.
	 * If called multiple times without shutting down, all but the first call will be ignored.
	 */
	virtual void Startup();

	/** Close the socket and kill the listening thread. */
	virtual void Shutdown();

protected:
	/**
	 * Get the IP address on which to listen for beacon messages.
	 * @param OutAddress The IP address.
	 * @return false if no valid address could be found, else true.
	 */
	virtual bool GetDiscoveryAddress(FIPv4Address& OutAddress) const = 0;

	/**
	 * Get the port on which to listen for beacon messages.
	 * @param OutAddress The port, or a negative value if no valid port could be found.
	 */
	virtual int32 GetDiscoveryPort() const = 0;

	/**
	 * Make a response to a beacon message.
	 * @param BeaconProtocolVersion The protocol version reported by the beacon message.
	 * @param InMessageData Array reader containing additional data after the standard beacon protocol version and identifier.
	 *						All data in this reader must be consumed or the message will be ignored.
	 * @param OutResponseData 
	 */
	virtual bool MakeBeaconResponse(uint8 BeaconProtocolVersion, FArrayReader& InMessageData, FArrayWriter& OutResponseData) const = 0;

private:
	//~ Begin FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	//~ End FRunnable implementation

	/** Called regularly to receive beacon messages. */
	void ReceiveBeaconMessages();

	/** Handle a message received on the beacon endpoint. */
	void HandleBeaconMessage(FArrayReader& MessageData, TSharedRef<FInternetAddr> Source);

private:
	/** A short name for this beacon for debugging purposes. */
	FString Description;

	/** A series of bytes that uniquely identify messages intended for this beacon receiver. */
	TArray<uint8> ProtocolIdentifier;

	/** The protocol version number with which to reply to beacon messages. */
	uint8 ProtocolVersion;

	/** Identifier for this engine instance. Sent in responses to differentiate engine instances even if they're on the same machine. */
	FGuid Guid;

	/** Socket used to listen for and reply to beacon messages. */
	FSocket* Socket = nullptr;

	/** Flag indicating that the beacon thread should be running. */
	TAtomic<bool> bIsRunning = false;

	/** The thread to receive beacon messages on. */
	TUniquePtr<FRunnableThread> Thread = nullptr;
};