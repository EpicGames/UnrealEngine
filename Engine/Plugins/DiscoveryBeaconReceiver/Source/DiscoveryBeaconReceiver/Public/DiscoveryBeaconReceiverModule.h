// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

DISCOVERYBEACONRECEIVER_API DECLARE_LOG_CATEGORY_EXTERN(LogDiscoveryBeaconReceiver, Log, All);

class FDiscoveryBeaconReceiverModule : public IModuleInterface
{
};